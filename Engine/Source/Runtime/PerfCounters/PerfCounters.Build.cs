// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PerfCounters : ModuleRules
{
	public PerfCounters(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Json",
				"HTTPServer"
			}
		);
	}
}
