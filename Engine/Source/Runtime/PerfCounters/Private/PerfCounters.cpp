// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerfCounters.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "HAL/PlatformMemoryHelpers.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringOutputDevice.h"
#include "Serialization/JsonWriter.h"
#include "Stats/Stats.h"
#include "ZeroLoad.h"

#include "HttpServerModule.h"
#include "IHttpRouter.h"
#include "HttpServerRequest.h"
#include "HttpServerResponse.h"
#include "HttpRequestHandler.h"
#include "HttpServerConstants.h"

#define JSON_ARRAY_NAME					TEXT("PerfCounters")
#define JSON_PERFCOUNTER_NAME			TEXT("Name")
#define JSON_PERFCOUNTER_SIZE_IN_BYTES	TEXT("SizeInBytes")

#define PERF_COUNTER_CONNECTION_TIMEOUT 5.0f

FPerfCounters::FPerfCounters(const FString& InUniqueInstanceId)
	: UniqueInstanceId(InUniqueInstanceId)
	, InternalCountersUpdateInterval(60)
	, ZeroLoadThread(nullptr)
	, ZeroLoadRunnable(nullptr)
{
}

FPerfCounters::~FPerfCounters()
{
	if (HttpRouter)
	{
		HttpRouter->UnbindRoute(StatsRouteHandle);
		HttpRouter->UnbindRoute(ExecRouteHandle);
	}
}

bool FPerfCounters::Initialize()
{
	float ConfigInternalCountersUpdateInterval = 60.0;
	if (GConfig->GetFloat(TEXT("PerfCounters"), TEXT("InternalCountersUpdateInterval"), ConfigInternalCountersUpdateInterval, GEngineIni))
	{
		InternalCountersUpdateInterval = ConfigInternalCountersUpdateInterval;
	}
	LastTimeInternalCountersUpdated = static_cast<float>(FPlatformTime::Seconds()) - InternalCountersUpdateInterval * FMath::FRand();	// randomize between servers

	// get the requested port from the command line (if specified)
	const int32 StatsPort = IPerfCountersModule::GetHTTPStatsPort();
	if (StatsPort < 0)
	{
		UE_LOGF(LogPerfCounters, Log, "FPerfCounters JSON socket disabled.");
		return true;
	}

	// Get an IHttpRouter on the command-line designated port
	HttpRouter = FHttpServerModule::Get().GetHttpRouter(StatsPort, /* bFailOnBindFailure = */ true);
	if (!HttpRouter)
	{
		UE_LOGF(LogPerfCounters, Error, 
			"FPerfCounters unable to bind to specified statsPort [%d]", StatsPort);
		return false;
	}

	// Register a handler for /stats
	TWeakPtr<FPerfCounters> WeakThisPtr(AsShared());
    StatsRouteHandle = HttpRouter->BindRoute(FHttpPath("/stats"), EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateSP(this, &FPerfCounters::ProcessStatsRequest));
	if(!StatsRouteHandle.IsValid())
	{
		UE_LOGF(LogPerfCounters, Error,
			"FPerfCounters unable bind route: /stats");
		return false;
	}

	// Register a handler for /exec
	ExecRouteHandle = HttpRouter->BindRoute(FHttpPath("/exec"), EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateSP(this, &FPerfCounters::ProcessExecRequest));
	if(!ExecRouteHandle.IsValid())
	{
		UE_LOGF(LogPerfCounters, Error,
			"FPerfCounters unable bind route: /exec");
		return false;
	}

	return true;
}

FString FPerfCounters::GetAllCountersAsJson()
{
	FString JsonStr;
	TSharedRef< TJsonWriter<> > Json = TJsonWriterFactory<>::Create(&JsonStr);
	Json->WriteObjectStart();
	for (const auto& It : PerfCounterMap)
	{
		const FJsonVariant& JsonValue = It.Value;
		switch (JsonValue.Format)
		{
		case FJsonVariant::String:
			Json->WriteValue(It.Key, JsonValue.StringValue);
			break;
		case FJsonVariant::Number:
			Json->WriteValue(It.Key, JsonValue.NumberValue);
			break;
		case FJsonVariant::Callback:
			if (JsonValue.CallbackValue.IsBound())
			{
				Json->WriteIdentifierPrefix(It.Key);
				JsonValue.CallbackValue.Execute(Json);
			}
			else
			{
				// write an explicit null since the callback is unbound and the implication is this would have been an object
				Json->WriteNull(It.Key);
			}
			break;
		case FJsonVariant::Null:
		default:
			// don't write anything since wash may expect a scalar
			break;
		}
	}
	Json->WriteObjectEnd();
	Json->Close();
	return JsonStr;
}

void FPerfCounters::ResetStatsForNextPeriod()
{
	UE_LOGF(LogPerfCounters, Verbose, "Clearing perf counters.");
	for (TMap<FString, FJsonVariant>::TIterator It(PerfCounterMap); It; ++It)
	{
		if (It.Value().Flags & IPerfCounters::Flags::Transient)
		{
			UE_LOGF(LogPerfCounters, Verbose, "  Removed '%ls'", *It.Key());
			It.RemoveCurrent();
		}
	}
};

bool FPerfCounters::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FPerfCounters_Tick);

	if (LIKELY(ZeroLoadThread != nullptr))
	{
		TickZeroLoad(DeltaTime);
	}

	TickSystemCounters(DeltaTime);

	// keep ticking
	return true;
}

void FPerfCounters::TickZeroLoad(float DeltaTime)
{
	checkf(ZeroLoadThread != nullptr, TEXT("FPerfCounters::TickZeroThread() called without a valid socket!"));

	TArray<FString> LogMessages;
	if (LIKELY(ZeroLoadThread->GetHitchMessages(LogMessages)))
	{
		for (const FString& LogMessage : LogMessages)
		{
			UE_LOGF(LogPerfCounters, Warning, "%ls", *LogMessage);
		}
	}
}


void FPerfCounters::TickSystemCounters(float DeltaTime)
{
	// set some internal perf stats ([RCL] FIXME 2015-12-08: move to a better place)
	float CurrentTime = static_cast<float>(FPlatformTime::Seconds());
	if (CurrentTime - LastTimeInternalCountersUpdated > InternalCountersUpdateInterval)
	{
		// get CPU stats first
		FCPUTime CPUStats = FPlatformTime::GetCPUTime();
		Set(TEXT("ProcessCPUUsageRelativeToCore"), CPUStats.CPUTimePctRelative);

		// memory
		FPlatformMemoryStats Stats = PlatformMemoryHelpers::GetFrameMemoryStats();
		Set(TEXT("AvailablePhysicalMemoryMB"), static_cast<uint64>(Stats.AvailablePhysical / (1024 * 1024)));
		Set(TEXT("AvailableVirtualMemoryMB"), static_cast<uint64>(Stats.AvailableVirtual / (1024 * 1024)));
		Set(TEXT("ProcessPhysicalMemoryMB"), static_cast<uint64>(Stats.UsedPhysical/ (1024 * 1024)));
		Set(TEXT("ProcessVirtualMemoryMB"), static_cast<uint64>(Stats.UsedVirtual / (1024 * 1024)));

		// disk space
		const FString LogFilename = FPlatformOutputDevices::GetAbsoluteLogFilename();
		uint64 TotalBytesOnLogDrive = 0, FreeBytesOnLogDrive = 0;
		if (FPlatformMisc::GetDiskTotalAndFreeSpace(LogFilename, TotalBytesOnLogDrive, FreeBytesOnLogDrive))
		{
			Set(TEXT("FreeSpaceOnLogFileDiskInMB"), static_cast<uint64>(FreeBytesOnLogDrive / (1024 * 1024)));
		}

		LastTimeInternalCountersUpdated = CurrentTime;
	}
}

bool FPerfCounters::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// ignore everything that doesn't start with PerfCounters
	if (!FParse::Command(&Cmd, TEXT("perfcounters")))
	{
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("clear")))
	{
		ResetStatsForNextPeriod();
		return true;
	}

	return false;
}

bool FPerfCounters::ProcessStatsRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	auto ResponseBody = GetAllCountersAsJson();
	auto Response = FHttpServerResponse::Create(ResponseBody, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}

bool FPerfCounters::ProcessExecRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	const FString* ExecCmd = Request.QueryParams.Find(TEXT("C"));
	if (ExecCmd)
	{
		if (ExecCmdCallback.IsBound())
		{
			FStringOutputDevice StringOutDevice;
			StringOutDevice.SetAutoEmitLineTerminator(true);

			const bool bExecCommandSuccess = ExecCmdCallback.Execute(*ExecCmd, StringOutDevice);
			if (bExecCommandSuccess)
			{
				auto Response = FHttpServerResponse::Create(StringOutDevice, TEXT("text/text"));
				OnComplete(MoveTemp(Response));
			}
			else
			{
				auto Response = FHttpServerResponse::Error(EHttpServerResponseCodes::NotSupported, StringOutDevice);
				OnComplete(MoveTemp(Response));
			}
		}
		else
		{
			auto Response = FHttpServerResponse::Error(EHttpServerResponseCodes::NotSupported,
				TEXT("exec handler not found"));
			OnComplete(MoveTemp(Response));
		}
	}
	else
	{
		auto Response = FHttpServerResponse::Error(EHttpServerResponseCodes::NotSupported,
			TEXT("exec missing query command (c=MyCommand)"));
		OnComplete(MoveTemp(Response));
	}

	return true;
}

double FPerfCounters::GetNumber(const FString& Name, double DefaultValue)
{
	FJsonVariant * JsonValue = PerfCounterMap.Find(Name);
	if (JsonValue == nullptr)
	{
		return DefaultValue;
	}

	if (JsonValue->Format != FJsonVariant::Number)
	{
		UE_LOGF(LogPerfCounters, Warning, "Attempting to get PerfCounter '%ls' as number, but it is not (Json format=%d). Default value %f will be returned", 
			*Name, static_cast<int32>(JsonValue->Format), DefaultValue);

		return DefaultValue;
	}

	return JsonValue->NumberValue;
}

void FPerfCounters::SetNumber(const FString& Name, double Value, uint32 Flags)
{
	FJsonVariant& JsonValue = PerfCounterMap.FindOrAdd(Name);
	JsonValue.Format = FJsonVariant::Number;
	JsonValue.Flags = Flags;
	JsonValue.NumberValue = Value;
}

void FPerfCounters::SetString(const FString& Name, const FString& Value, uint32 Flags)
{
	FJsonVariant& JsonValue = PerfCounterMap.FindOrAdd(Name);
	JsonValue.Format = FJsonVariant::String;
	JsonValue.Flags = Flags;
	JsonValue.StringValue = Value;
}

void FPerfCounters::SetJson(const FString& Name, const FProduceJsonCounterValue& InCallback, uint32 Flags)
{
	FJsonVariant& JsonValue = PerfCounterMap.FindOrAdd(Name);
	JsonValue.Format = FJsonVariant::Callback;
	JsonValue.Flags = Flags;
	JsonValue.CallbackValue = InCallback;
}

bool FPerfCounters::StartMachineLoadTracking()
{
	return StartMachineLoadTracking(30.0, TArray<double>());
}

bool FPerfCounters::StartMachineLoadTracking(double TickRate, const TArray<double>& FrameTimeHistogramBucketsMs)
{
	if (UNLIKELY(ZeroLoadRunnable != nullptr || ZeroLoadThread != nullptr))
	{
		UE_LOGF(LogPerfCounters, Warning, "Machine load tracking has already been started.");
		return false;
	}

	// support the legacy pathway
	ZeroLoadThread = FrameTimeHistogramBucketsMs.Num() == 0 ? new FZeroLoad(TickRate) : new FZeroLoad(TickRate, FrameTimeHistogramBucketsMs);
	ZeroLoadRunnable = FRunnableThread::Create(ZeroLoadThread, TEXT("ZeroLoadThread"), 0, TPri_Normal);

	if (UNLIKELY(ZeroLoadRunnable == nullptr))
	{
		UE_LOGF(LogPerfCounters, Warning, "Failed to create zero load thread.");

		delete ZeroLoadThread;
		ZeroLoadThread = nullptr;

		return false;
	}

	return true;
}

bool FPerfCounters::StopMachineLoadTracking()
{
	if (UNLIKELY(ZeroLoadRunnable == nullptr || ZeroLoadThread == nullptr))
	{
		UE_LOGF(LogPerfCounters, Warning, "Machine load tracking has already been stopped.");
		return false;
	}

	// this will first call Stop()
	if (!ZeroLoadRunnable->Kill(true))
	{
		UE_LOGF(LogPerfCounters, Warning, "Could not kill zero-load thread, crash imminent.");
	}

	// set the its histogram as one of counters
	FHistogram ZeroLoadFrameTimes;
	if (ZeroLoadThread->GetFrameTimeHistogram(ZeroLoadFrameTimes))
	{
		PerformanceHistograms().Add(IPerfCounters::Histograms::ZeroLoadFrameTime, ZeroLoadFrameTimes);
	}

	delete ZeroLoadRunnable;
	ZeroLoadRunnable = nullptr;

	delete ZeroLoadThread;
	ZeroLoadThread = nullptr;

	return true;
}

bool FPerfCounters::ReportUnplayableCondition(const FString& ConditionDescription)
{
	FString UnplayableConditionFile(FPaths::Combine(*FPaths::ProjectSavedDir(), *FString::Printf(TEXT("UnplayableConditionForPid_%d.txt"), FPlatformProcess::GetCurrentProcessId())));

	FArchive* ReportFile = IFileManager::Get().CreateFileWriter(*UnplayableConditionFile);
	if (UNLIKELY(ReportFile == nullptr))
	{
		return false;
	}

	// include description for debugging
	FTCHARToUTF8 Converter(*FString::Printf(TEXT("Unplayable condition encountered: %s\n"), *ConditionDescription));
	ReportFile->Serialize(reinterpret_cast<void *>(const_cast<char *>(reinterpret_cast<const char *>(Converter.Get()))), Converter.Length());

	ReportFile->Close();
	delete ReportFile;

	return true;
}
