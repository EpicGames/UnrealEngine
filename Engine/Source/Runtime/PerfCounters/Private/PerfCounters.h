// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "PerfCountersModule.h"
#include "Containers/Ticker.h"
#include "ProfilingDebugging/Histogram.h"

#include "HttpResultCallback.h"
#include "HttpPath.h"
#include "HttpRouteHandle.h"

class IHttpRouter;
struct FHttpServerRequest;
class FZeroLoad;
class FInternetAddr;

DECLARE_LOG_CATEGORY_EXTERN(LogPerfCounters, Log, All);

class FPerfCounters 
	: public FTSTickerObjectBase
	, public FSelfRegisteringExec
	, public IPerfCounters
	, public TSharedFromThis<FPerfCounters>
{
public:

	FPerfCounters(const FString& InUniqueInstanceId);
	virtual ~FPerfCounters();

	/** Initializes this instance from JSON config. */
	bool Initialize();

	/** FTSTickerObjectBase */
	virtual bool Tick(float DeltaTime) override;

	//~ Begin IPerfCounters Interface
	const FString& GetInstanceName() const override { return UniqueInstanceId; }
	virtual double GetNumber(const FString& Name, double DefaultValue = 0.0) override;
	virtual void SetNumber(const FString& Name, double Value, uint32 Flags) override;
	virtual void SetString(const FString& Name, const FString& Value, uint32 Flags) override;
	virtual void SetJson(const FString& Name, const FProduceJsonCounterValue& InCallback, uint32 Flags) override;
	virtual FPerfCounterExecCommandCallback& OnPerfCounterExecCommand() override { return ExecCmdCallback; }
	virtual const TMap<FString, FJsonVariant>& GetAllCounters() override { return PerfCounterMap; }
	virtual FString GetAllCountersAsJson() override;
	virtual void ResetStatsForNextPeriod() override;
	virtual TPerformanceHistogramMap& PerformanceHistograms() override { return PerformanceHistogramMap; }
	// Legacy load tracking, which is raw frame time, not overshoot.
	virtual bool StartMachineLoadTracking() override;
	// If OvershootBuckets is empty, will use legacy load tracking values, which is raw frame time, not overshoot.
	virtual bool StartMachineLoadTracking(double TickRate, const TArray<double>& FrameTimeHistogramBucketsMs) override;
	virtual bool StopMachineLoadTracking();
	virtual bool ReportUnplayableCondition(const FString& ConditionDescription);
	//~ Begin IPerfCounters Interface end

protected:
	//~ Begin Exec Interface
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	//~ End Exec Interface

private:
	
	void TickZeroLoad(float DeltaTime);
	void TickSystemCounters(float DeltaTime);

	/**
	 * Processes a /stats request
	 *
	 * @param Request The incoming request
	 * @param OnComplete The invokable response result callback
	 * @return true if this request was handled herein, false otherwise
	 */
	bool ProcessStatsRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/**
	 * Process an /exec request
	 *
	 * @param Request The incoming request
	 * @param OnComplete The invokable response result callback
	 * @return true if this request was handled herein, false otherwise
	 */
	bool ProcessExecRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/** Unique name of this instance */
	FString UniqueInstanceId;

	/** Interval (in seconds) to update internal counters, added by the module itself, a config value */
	float InternalCountersUpdateInterval;

	/** Last time internal counters were updated */
	float LastTimeInternalCountersUpdated;

	/** Map of all known performance counters */
	TMap<FString, FJsonVariant>  PerfCounterMap;

	/** Bound callback for script command execution */
	FPerfCounterExecCommandCallback ExecCmdCallback;

	/** Map of performance histograms. */
	TPerformanceHistogramMap PerformanceHistogramMap;

	/** Data of zero-load thread (used for measuring machine load). */
	FZeroLoad* ZeroLoadThread;

	/** Zero-load thread runnable */
	FRunnableThread* ZeroLoadRunnable;

	/** Http request/response interface */
	TSharedPtr<IHttpRouter> HttpRouter = nullptr;

	/** Route handle for /stats binding */
	FHttpRouteHandle StatsRouteHandle = nullptr;

	/** Route handle for /exec binding */
	FHttpRouteHandle ExecRouteHandle = nullptr;
};
