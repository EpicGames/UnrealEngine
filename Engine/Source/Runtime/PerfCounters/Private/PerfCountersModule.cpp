// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerfCountersModule.h"
#include "HAL/PlatformProcess.h"
#include "PerfCounters.h"
#include "Misc/CommandLine.h"
#include "Misc/Fork.h"

class FPerfCountersModule : public IPerfCountersModule
{
private:

	/** All created perf counter instances from this module */
	TSharedPtr<FPerfCounters> PerfCountersSingleton = nullptr;

public:

	FPerfCountersModule() 
	{
	}

	virtual void ShutdownModule() override
	{
		PerfCountersSingleton.Reset();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}

	IPerfCounters* GetPerformanceCounters() const
	{
		return PerfCountersSingleton.Get();
	}

	IPerfCounters* CreatePerformanceCounters(const FString& UniqueInstanceId) override
	{
		if (PerfCountersSingleton)
		{
			UE_LOGF(LogPerfCounters, Display, "CreatePerformanceCounters: instance already exists, new instance not created.");
			return PerfCountersSingleton.Get();
		}

		FString InstanceUID = UniqueInstanceId;
		if (InstanceUID.IsEmpty())
		{
			InstanceUID = FString::Printf(TEXT("perfcounters-of-pid-%d"), FPlatformProcess::GetCurrentProcessId());
		}
 
		PerfCountersSingleton = MakeShared<FPerfCounters>(InstanceUID);
		if (!PerfCountersSingleton->Initialize())
		{
			UE_LOGF(LogPerfCounters, Warning, "CreatePerformanceCounters: could not create perfcounters");
			return nullptr;
		}

		return PerfCountersSingleton.Get();
	}
};

IMPLEMENT_MODULE(FPerfCountersModule, PerfCounters)
DEFINE_LOG_CATEGORY(LogPerfCounters);

const FName IPerfCounters::Histograms::FrameTime(TEXT("FrameTime"));
const FName IPerfCounters::Histograms::FrameTimePeriodic(TEXT("FrameTimePeriodic"));
const FName IPerfCounters::Histograms::FrameTimeWithoutSleep(TEXT("FrameTimeWithoutSleep"));
const FName IPerfCounters::Histograms::ServerReplicateActorsTime(TEXT("ServerReplicateActorsTime"));
const FName IPerfCounters::Histograms::SleepTime(TEXT("SleepTime"));
const FName IPerfCounters::Histograms::ZeroLoadFrameTime(TEXT("ZeroLoadFrameTime"));

int32 IPerfCountersModule::GetHTTPStatsPort()
{
	int32 StatsPort = -1;
	FParse::Value(FCommandLine::Get(), TEXT("statsPort="), StatsPort);

	if (StatsPort > 0 && FParse::Param(FCommandLine::Get(), TEXT("forkAutoStatsPort")))
	{
		const int32 ChildProcessIdx = FForkProcessHelper::GetForkedChildProcessIndex();
		StatsPort += ChildProcessIdx;
		UE_LOGF(LogPerfCounters, Display, "ForkAutoStatsPort assigned port %d to child process %d", StatsPort, ChildProcessIdx);
	}

	return StatsPort;
}