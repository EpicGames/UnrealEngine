// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZeroLoad.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"

DEFINE_LOG_CATEGORY_STATIC(LogZeroLoad, Log, All);

FZeroLoad::FZeroLoad(double InTickRate)
	: TickRate(InTickRate)
{
	TickTimeHistogram.InitHitchTracking();
}

FZeroLoad::FZeroLoad(double InTickRate, const TArray<double>& FrameTimeHistogramBucketsMs)
	: TickRate(InTickRate)
{
	TickTimeHistogram.InitFromArray(FrameTimeHistogramBucketsMs);
}

bool FZeroLoad::Init()
{
	HitchMessagesToBeLogged.Empty();	// unguarded since nothing else should be accessing it at this point

	return true;
}

void FZeroLoad::Exit()
{
}

uint32 FZeroLoad::Run()
{
	const double SecondsToSleep = (TickRate > 0.0) ? (1.0 / TickRate) : 0.033;	// default to 30 HZ
	const double PermissibleDelayInSeconds = 0.01; // server kernel may be configured to tick at 100 HZ, so forgive scheduler some imprecision

	double PreviousTick = FPlatformTime::Seconds();
	while (!ExitRequest.GetValue())
	{
		FPlatformProcess::SleepNoStats(static_cast<float>(SecondsToSleep));

		const double CurrentTick = FPlatformTime::Seconds();
		const double TickDuration = CurrentTick - PreviousTick;
		const double TickDurationInMs = TickDuration * 1000.0;
		PreviousTick = CurrentTick;

		TickTimeHistogram.AddMeasurement(TickDurationInMs);

		// if we exceeded our sleep time by too much, log as a hitch
		if (TickDuration > SecondsToSleep + PermissibleDelayInSeconds)
		{
			// warning: this can block, adding to spurious hitch
			AddHitchMessage(TickDurationInMs);
		}
	}

	return 0;
}

void FZeroLoad::Stop()
{
	ExitRequest.Set(1);
}

void FZeroLoad::AddHitchMessage(double HitchDurationInMs)
{
	FString Message = FString::Printf(TEXT("Zero-load thread experienced hitch of %f ms at %s"),
		HitchDurationInMs,
		*FDateTime::UtcNow().ToString()
		);

	{
		FScopeLock HitchMessagesGuard(&HitchMessagesLock);
		HitchMessagesToBeLogged.Add(Message);
	}
}

bool FZeroLoad::GetHitchMessages(TArray<FString>& OutArray)
{
	FScopeLock HitchMessagesGuard(&HitchMessagesLock);

	bool bResult = HitchMessagesToBeLogged.Num() != 0;
	if (bResult)
	{
		OutArray = HitchMessagesToBeLogged;
		HitchMessagesToBeLogged.Reset();
	}

	return bResult;
}

bool FZeroLoad::GetFrameTimeHistogram(FHistogram& OutHistogram)
{
	checkf(ExitRequest.GetValue() == 1, TEXT("FZeroLoad::GetFrameTimeHistogram() is called while zero load thread is still running!"));

	OutHistogram = TickTimeHistogram;
	return true;
}

