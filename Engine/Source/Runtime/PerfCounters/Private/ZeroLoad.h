// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "ProfilingDebugging/Histogram.h"
#include "HAL/Runnable.h"

class FZeroLoad : public FRunnable
{
	/** signal request to stop and exit thread */
	FThreadSafeCounter ExitRequest;

	/** Tick frequency, HZ */
	double TickRate;

	/** Histogram of thread loop times, not accessed outside of the thread */
	FHistogram TickTimeHistogram;

	/** Guards access to hitch messages */
	FCriticalSection HitchMessagesLock;

	/** Hitches that have not yet been logged */
	TArray<FString> HitchMessagesToBeLogged;

	/** Adds a message to log. We avoid calling UE_LOG directly since this can add locks at unpredictable times. */
	void AddHitchMessage(double HitchDurationInMs);

public:

	// FRunnable interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	// end of FRunnable interface

	/** Gets messages to log - can block. */
	bool GetHitchMessages(TArray<FString>& OutArray);

	/** Gets frame time histogram - not guarded, should only be accessed after thread has stopped! */
	bool GetFrameTimeHistogram(FHistogram& OutHistogram);

	FZeroLoad(double InTickRate);
	FZeroLoad(double InTickRate, const TArray<double>& FrameTimeHistogramBucketsMs);
	virtual ~FZeroLoad() = default;
};
