// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FName;
class FOutputDevice;
struct FHistogram;
template <class CharType, class PrintPolicy>
class TJsonWriter;
template <class CharType>
struct TPrettyJsonPrintPolicy;
template <class CharType> struct TPrettyJsonPrintPolicy;

typedef TSharedRef< TJsonWriter<TCHAR,TPrettyJsonPrintPolicy<TCHAR> > > FPrettyJsonWriter;
struct FHistogram;

/**
 * Delegate called for a given counter to generate custom json at the time the query is made
 *
 * @param JsonWriter json output writer
 */
DECLARE_DELEGATE_OneParam(FProduceJsonCounterValue, const FPrettyJsonWriter& /* JsonWriter */);

/**
 * Delegate called when an exec command has been passed in on via the query port
 *
 * @param ExecCmd string command to execute
 * @param Output output device to write the response back to the querying service
 * 
 * @return true if the call was handled, false otherwise
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FPerfCounterExecCommandCallback, const FString& /*ExecCmd*/, FOutputDevice& /*Output*/);

/**
 * A programming interface for setting/updating performance counters
 */
class IPerfCounters
{
public:

	enum Flags : uint32
	{
		/** Perf counter with this flag will be removed by "perfcounters clear" command */
		Transient = (1 << 0)
	};

	struct FJsonVariant
	{
		enum { Null, String, Number, Callback } Format;
		FString		StringValue;
		double		NumberValue;
		FProduceJsonCounterValue CallbackValue;
		uint32		Flags;

		FJsonVariant() : Format(Null), NumberValue(0) {}
	};

	/** Named engine-wide histograms */
	struct Histograms
	{
		/** Frame time histogram for the duration of the match. */
		static PERFCOUNTERS_API const FName FrameTime;
		/** Frame time histogram for shorter intervals. */
		static PERFCOUNTERS_API const FName FrameTimePeriodic;
		/** Frame time histogram (without sleep) for the duration of the match. */
		static PERFCOUNTERS_API const FName FrameTimeWithoutSleep;
		/** ServerReplicateActors time histogram for the duration of the match. */
		static PERFCOUNTERS_API const FName ServerReplicateActorsTime;
		/** Sleep time histogram for the duration of the match. */
		static PERFCOUNTERS_API const FName SleepTime;

		/** Zero load thread frame time histogram. */
		static PERFCOUNTERS_API const FName ZeroLoadFrameTime;
	};

	/** Array used to store performance histograms. */
	typedef TMap<FName, FHistogram>		TPerformanceHistogramMap;


	virtual ~IPerfCounters() = default;

	/** Get the unique identifier for this perf counter instance */
	virtual const FString& GetInstanceName() const = 0;

	/** Returns currently held value, or DefaultValue if none */
	virtual double GetNumber(const FString& Name, double DefaultValue = 0.0) = 0;

	/** Maps value to a numeric holder */
	virtual void SetNumber(const FString& Name, double Value, uint32 Flags = 0) = 0;

	/** Maps value to a string holder */
	virtual void SetString(const FString& Name, const FString& Value, uint32 Flags = 0) = 0;

	/** Make a callback so we can request more extensive types on demand (presumably backed by some struct locally) */
	virtual void SetJson(const FString& Name, const FProduceJsonCounterValue& Callback, uint32 Flags = 0) = 0;

	/** @return delegate called when an exec command is to be executed */
	virtual FPerfCounterExecCommandCallback& OnPerfCounterExecCommand() = 0;

	/** @return all perf counters as they are stored, with minimum conversion */
	virtual const TMap<FString, FJsonVariant>& GetAllCounters() = 0;

	/** @return all perf counters as JSON */
	virtual FString GetAllCountersAsJson() = 0;

	/** Clears transient perf counters, essentially marking beginning of a new stats period */
	virtual void ResetStatsForNextPeriod() = 0;

	/** Returns performance histograms for direct manipulation by the client code. */
	virtual TPerformanceHistogramMap& PerformanceHistograms() = 0;

	/** Starts tracking overall machine load. */
	virtual bool StartMachineLoadTracking() = 0;

	/** Starts tracking overall machine load (empty array will use default hitch buckets). */
	virtual bool StartMachineLoadTracking(double TickRate, const TArray<double>& FrameTimeHistogramBucketsMs) = 0;

	/** Stops tracking overall machine load. */
	virtual bool StopMachineLoadTracking() = 0;

	/** Reports an unplayable condition. */
	virtual bool ReportUnplayableCondition(const FString& ConditionDescription) = 0;

public:

	/** Get overloads */
	int32 Get(const FString& Name, int32 Val = 0)
	{
		return static_cast<int32>(GetNumber(Name, static_cast<double>(Val)));
	}

	uint32 Get(const FString& Name, uint32 Val = 0)
	{
		return static_cast<uint32>(GetNumber(Name, static_cast<double>(Val)));
	}

	float Get(const FString& Name, float Val = 0.0f)
	{
		return static_cast<float>(GetNumber(Name, static_cast<double>(Val)));
	}

	double Get(const FString& Name, double Val = 0.0)
	{
		return GetNumber(Name, Val);
	}

	/** @brief Convenience method for incrementing a transient counter 
	 *
	 *  @param Name the name of the counter
	 *  @param Add value of the increment (will be added to the counter, can be negative)
	 *  @param DefaultValue if the counter did not exist or was cleared, this is what it will be initialized to before performing the addition
	 *  @param Flags flags for the counter
	 *
	 *  @return current value (i.e. after the increment)
	 */
	int32 Increment(const FString & Name, int32 Add = 1, int32 DefaultValue = 0, uint32 Flags = IPerfCounters::Flags::Transient)
	{
		int32 CurrentValue = Get(Name, DefaultValue) + Add;
		Set(Name, CurrentValue, Flags);
		return CurrentValue;
	}

	/** Set overloads (use these) */

	void Set(const FString& Name, int32 Val, uint32 Flags = 0)
	{
		SetNumber(Name, (double)Val, Flags);
	}

	void Set(const FString& Name, uint32 Val, uint32 Flags = 0)
	{
		SetNumber(Name, (double)Val, Flags);
	}

	void Set(const FString& Name, float Val, uint32 Flags = 0)
	{
		SetNumber(Name, (double)Val, Flags);
	}

	void Set(const FString& Name, double Val, uint32 Flags = 0)
	{
		SetNumber(Name, Val, Flags);
	}

	void Set(const FString& Name, int64 Val, uint32 Flags = 0)
	{
		SetString(Name, FString::Printf(TEXT("%lld"), Val), Flags);
	}

	void Set(const FString& Name, uint64 Val, uint32 Flags = 0)
	{
		SetString(Name, FString::Printf(TEXT("%llu"), Val), Flags);
	}

	void Set(const FString& Name, const FString& Val, uint32 Flags = 0)
	{
		SetString(Name, Val, Flags);
	}

	void Set(const FString& Name, const FProduceJsonCounterValue& Callback, uint32 Flags = 0)
	{
		SetJson(Name, Callback, Flags);
	}
};


/**
 * The public interface to this module
 */
class IPerfCountersModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPerfCountersModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IPerfCountersModule >("PerfCounters");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PerfCounters");
	}

	/** Returns the HTTP port to bind for the stats request. Returns -1 if none were specified */
	static PERFCOUNTERS_API int32 GetHTTPStatsPort();

	/**
	 * @return the currently initialized / in use perf counters 
	 */
	virtual IPerfCounters* GetPerformanceCounters() const = 0;

	/** 
	 * Creates and initializes the performance counters object
	 *
	 * @param UniqueInstanceId		optional parameter that allows to assign a known name for this set of counters (a default one that will include process id will be provided if not given)
	 *
	 * @return IPerfCounters object (should be explicitly deleted later), or nullptr if failed
	 */
	virtual IPerfCounters* CreatePerformanceCounters(const FString& UniqueInstanceId = TEXT("")) = 0;
};
